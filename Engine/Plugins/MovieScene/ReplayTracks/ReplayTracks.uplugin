{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Replay Tracks (Experimental)",
	"Description" : "Sequence tracks for playing recorded gameplay",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "ReplayTracks",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "ReplayTracksEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		}
	]
}

