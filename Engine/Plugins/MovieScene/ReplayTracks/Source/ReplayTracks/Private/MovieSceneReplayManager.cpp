// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneReplayManager.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/WorldSettings.h"

/**
 * Utility actor that lets FMovieSceneDefaultReplayBroker keep the time dilation to the
 * desired value every frame. See comment in FMovieSceneDefaultReplayBroker::OnReplayStarted.
 */
class FMovieSceneDefaultReplayBrokerTicker : public FTickableGameObject
{
public:
	static FMovieSceneDefaultReplayBrokerTicker& Get(UWorld* World);
	static void Discard(UWorld* World);

	virtual TStatId GetStatId() const override {  RETURN_QUICK_DECLARE_CYCLE_STAT(FMovieSceneDefaultReplayBrokerTicker, STATGROUP_Tickables); }
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; }
	virtual bool IsTickable() const override { return bEnabled; }
	virtual bool IsTickableWhenPaused() const override { return true; }
	virtual void Tick(float DeltaTime) override;

private:
	friend class FMovieSceneDefaultReplayBroker;

	FMovieSceneDefaultReplayBrokerTicker();

	static FMovieSceneDefaultReplayBrokerTicker Instance;

	UWorld* World = nullptr;
	bool bEnabled = false;
	float TimeDilation = 1.f;
};

FMovieSceneDefaultReplayBrokerTicker FMovieSceneDefaultReplayBrokerTicker::Instance;

FMovieSceneDefaultReplayBrokerTicker& FMovieSceneDefaultReplayBrokerTicker::Get(UWorld* World)
{
	ensure(World != nullptr);
	ensure(Instance.World == nullptr || Instance.World == World);
	Instance.World = World;
	return Instance;
}

void FMovieSceneDefaultReplayBrokerTicker::Discard(UWorld* World)
{
	ensure(Instance.World == World);
	Instance.World = nullptr;
	Instance.bEnabled = false;
}

FMovieSceneDefaultReplayBrokerTicker::FMovieSceneDefaultReplayBrokerTicker()
{
}

void FMovieSceneDefaultReplayBrokerTicker::Tick(float DeltaTime)
{
	if (ensure(World))
	{
		// Calling SetTimeDilation (instead of setting TimeDilation directly) lets the
		// world settings clamp it to sane values. In particular, some games don't support
		// a time dilation of zero, and will instead set it to a very very small value.
		World->GetWorldSettings()->SetTimeDilation(TimeDilation);
	}
}

/**
 * Default implementation for how to manage the replay in a game.
 * It relies on a ticker actor to maintain the game's time dilation so that the replay is
 * correctly playing or paused.
 */
class FMovieSceneDefaultReplayBroker : public FMovieSceneReplayBroker
{
public:
	FMovieSceneDefaultReplayBroker() {}

private:
	virtual bool SupportsWorld(const UWorld* InWorld) const override { return true; }
	virtual bool CanStartReplay(const UWorld* InWorld) const override;
	virtual void OnReplayStarted(UWorld* InWorld) override;
	virtual void OnReplayPlay(UWorld* InWorld) override;
	virtual void OnReplayPause(UWorld* InWorld) override;
	virtual void OnReplayStopped(UWorld* InWorld) override;
};

bool FMovieSceneDefaultReplayBroker::CanStartReplay(const UWorld* InWorld) const
{
	if (APlayerController* PlayerController = InWorld->GetFirstPlayerController())
	{
		if (ASpectatorPawn* SpectatorPawn = PlayerController->GetSpectatorPawn())
		{
			return true;
		}
	}
	return false;
}

void FMovieSceneDefaultReplayBroker::OnReplayStarted(UWorld* InWorld)
{
	// We need an object that can set the time dilation every tick because otherwise the replay
	// will cause it to be overwritten and start playing on its own. In particular, it happens
	// any time the replay rewinds time, which ends up calling RewindForReplay on the UWorldSettings
	// which resets time dilation to 1.
 	FMovieSceneDefaultReplayBrokerTicker& Ticker = FMovieSceneDefaultReplayBrokerTicker::Get(InWorld);
	Ticker.bEnabled = true;
}

void FMovieSceneDefaultReplayBroker::OnReplayPlay(UWorld* InWorld)
{
 	FMovieSceneDefaultReplayBrokerTicker& Ticker = FMovieSceneDefaultReplayBrokerTicker::Get(InWorld);
	Ticker.TimeDilation = 1.f;
}

void FMovieSceneDefaultReplayBroker::OnReplayPause(UWorld* InWorld)
{
 	FMovieSceneDefaultReplayBrokerTicker& Ticker = FMovieSceneDefaultReplayBrokerTicker::Get(InWorld);
	Ticker.TimeDilation = 0.f;
}

void FMovieSceneDefaultReplayBroker::OnReplayStopped(UWorld* InWorld)
{
 	FMovieSceneDefaultReplayBrokerTicker::Discard(InWorld);

	// Restore time dilation to normal if we were paused.
	InWorld->GetWorldSettings()->SetTimeDilation(1.f);
}

FMovieSceneReplayManager& FMovieSceneReplayManager::Get()
{
	static FMovieSceneReplayManager Instance;
	return Instance;
}

FMovieSceneReplayManager::FMovieSceneReplayManager()
{
	RegisterBroker<FMovieSceneDefaultReplayBroker>();
}

bool FMovieSceneReplayManager::IsReplayArmed() const
{
	return bIsReplayArmed;
}

void FMovieSceneReplayManager::ArmReplay()
{
	if (ensure(!bIsReplayArmed))
	{
		bIsReplayArmed = true;
	}
}

void FMovieSceneReplayManager::DisarmReplay()
{
	if (ensure(bIsReplayArmed))
	{
		bIsReplayArmed = false;
	}
}

bool FMovieSceneReplayManager::UnregisterBroker(FMovieSceneReplayBrokerHandle InHandle)
{
	if (ensure(Brokers.IsValidIndex(InHandle.Value)))
	{
		// This will destroy the inline value and therefore destroy the broker object inside.
		Brokers.RemoveAt(InHandle.Value);
		return true;
	}
	return false;
}

FMovieSceneReplayBroker* FMovieSceneReplayManager::FindBroker(const UWorld* InWorld) const
{
	// Search from last to first to give the priority to the last registered brokers.
	// This means the default broker will be queried last.
	int32 Index = Brokers.FindLastByPredicate([InWorld](const TInlineValue<FMovieSceneReplayBroker>& It)
			{
				check(It.IsValid());
				return It->SupportsWorld(InWorld);
			});
	return (Index != INDEX_NONE) ? Brokers[Index].GetPtr() : nullptr;
}
