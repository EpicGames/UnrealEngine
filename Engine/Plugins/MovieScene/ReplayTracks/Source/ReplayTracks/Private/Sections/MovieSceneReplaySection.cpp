// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneReplaySection.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "Systems/MovieSceneReplaySystem.h"

UMovieSceneReplaySection::UMovieSceneReplaySection(const FObjectInitializer& ObjInitializer)
	: Super(ObjInitializer)
{
}

void UMovieSceneReplaySection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;
	
	const FReplayComponentTypes* ReplayComponents = FReplayComponentTypes::Get();
	
	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(ReplayComponents->Replay, this)
		);
}
