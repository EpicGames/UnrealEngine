// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneReplaySystem.h"
#include "Engine/DemoNetDriver.h"
#include "Engine/GameInstance.h"
#include "Engine/World.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneRootInstantiatorSystem.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "GameDelegates.h"
#include "GameFramework/SpectatorPawn.h"
#include "GameFramework/SpectatorPawnMovement.h"
#include "GameFramework/WorldSettings.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneReplayManager.h"
#include "Sections/MovieSceneReplaySection.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "ReplaySubsystem.h"
#include "TimerManager.h"

namespace UE
{
namespace MovieScene
{

static TUniquePtr<FReplayComponentTypes> GReplayComponentTypes;

FReplayComponentTypes* FReplayComponentTypes::Get()
{
	if (!GReplayComponentTypes.IsValid())
	{
		GReplayComponentTypes.Reset(new FReplayComponentTypes);
	}
	return GReplayComponentTypes.Get();
}

FReplayComponentTypes::FReplayComponentTypes()
{
	using namespace UE::MovieScene;

	FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();

	ComponentRegistry->NewComponentType(&Replay, TEXT("Replay"));

	ComponentRegistry->Factories.DuplicateChildComponent(Replay);
}

} // namespace MovieScene
} // namespace UE

FDelegateHandle UMovieSceneReplaySystem::PreLoadMapHandle;
FDelegateHandle UMovieSceneReplaySystem::PostLoadMapHandle;
FDelegateHandle UMovieSceneReplaySystem::EndPlayMapHandle;
FTimerHandle UMovieSceneReplaySystem::ReEvaluateHandle;

UMovieSceneReplaySystem::UMovieSceneReplaySystem(const FObjectInitializer& ObjInit)
	: UMovieSceneEntitySystem(ObjInit)
{
	using namespace UE::MovieScene;

	const FReplayComponentTypes* ReplayComponents = FReplayComponentTypes::Get();
	RelevantComponent = ReplayComponents->Replay;

	Phase = ESystemPhase::Instantiation | ESystemPhase::Evaluation;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneRootInstantiatorSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneReplaySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	// Bail out if we're not in a PIE/Game session... we can't replay stuff in editor.
	UWorld* OwningWorld = GetWorld();
	if (OwningWorld == nullptr || (OwningWorld->WorldType != EWorldType::Game && OwningWorld->WorldType != EWorldType::PIE))
	{
		return;
	}

	using namespace UE::MovieScene;

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();
	ESystemPhase CurrentPhase = Runner->GetCurrentPhase();

	if (CurrentPhase == ESystemPhase::Instantiation)
	{
		OnRunInstantiation();
	}
	else if (CurrentPhase == ESystemPhase::Evaluation)
	{
		OnRunEvaluation();
	}
}

void UMovieSceneReplaySystem::OnRunInstantiation()
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FReplayComponentTypes* ReplayComponents = FReplayComponentTypes::Get();

	// Check if we have any previously active replay.
	FReplayInfo PreviousReplayInfo;
	if (CurrentReplayInfos.Num() > 0)
	{
		PreviousReplayInfo = CurrentReplayInfos[0];
	}

	// Update our list of active replays.
	auto RemoveOldReplayInfos = [this](const FInstanceHandle& InstanceHandle, const FReplayComponentData& ReplayData)
	{
		if (ReplayData.Section != nullptr)
		{
			const FReplayInfo Key{ ReplayData.Section, InstanceHandle };
			const int32 Removed = CurrentReplayInfos.RemoveSingle(Key);
			ensure(Removed == 1);
		}
	};

	FEntityTaskBuilder()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(ReplayComponents->Replay)
		.FilterAll({ BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerEntity(&Linker->EntityManager, RemoveOldReplayInfos);

	auto AddNewReplayInfos = [this](const FInstanceHandle& InstanceHandle, const FReplayComponentData& ReplayData)
	{
		if (ReplayData.Section != nullptr)
		{
			const FReplayInfo Key{ ReplayData.Section, InstanceHandle };
			CurrentReplayInfos.Add(Key);
		}
	};

	FEntityTaskBuilder()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(ReplayComponents->Replay)
		.FilterAll({ BuiltInComponents->Tags.NeedsLink })
		.Iterate_PerEntity(&Linker->EntityManager, AddNewReplayInfos);

	// Check if we have any new current active replay.
	FReplayInfo NewReplayInfo;
	if (CurrentReplayInfos.Num() > 0)
	{
		NewReplayInfo = CurrentReplayInfos[0];
	}

	// If we have lost our previous replay, stop it.
	if (PreviousReplayInfo.IsValid() && (NewReplayInfo != PreviousReplayInfo))
	{
		if (bReplayActive)
		{
			StopReplay(PreviousReplayInfo);
		}
	}

	// If we have a new replay, start it... although it may have already been started, in which case we catch up with it.
	// This happens because the first time we get here, it runs the replay, which loads the replay's map, which wipes
	// everything. Once the replay map is loaded, the new evaluation gets us back here and it *looks* like we have a brand
	// new replay, but that's not really the case.
	if (NewReplayInfo.IsValid() && (NewReplayInfo != PreviousReplayInfo))
	{
		FMovieSceneReplayManager& Manager = FMovieSceneReplayManager::Get();
		if (Manager.GetReplayStatus() == EMovieSceneReplayStatus::Loading)
		{
			// We were loading a replay and caught back to it (see comment above). Let's mark it as active in this new
			// instance of the UMovieSceneReplaySystem and keep going.
			bReplayActive = true;

			// Set the manager to the "playing" status so that we can start playing right away in the evaluation phase.
			Manager.ReplayStatus = EMovieSceneReplayStatus::Playing;
		}
		else if (Manager.IsReplayArmed())
		{
			StartReplay(NewReplayInfo);
		}
	}

	// Initialize other stuff we need for evaluation.
	if (ShowFlagMotionBlur == nullptr)
	{
		ShowFlagMotionBlur = IConsoleManager::Get().FindConsoleVariable(TEXT("showflag.motionblur"));
	}
}

void UMovieSceneReplaySystem::OnRunEvaluation()
{
	using namespace UE::MovieScene;

	// Check if we have a valid active replay.
	if (CurrentReplayInfos.Num() == 0)
	{
		return;
	}
	if (!ensure(CurrentReplayInfos[0].IsValid()))
	{
		return;
	}

	// Bail out if we are still waiting for the game to be ready.
	FMovieSceneReplayManager& Manager = FMovieSceneReplayManager::Get();
	if (Manager.ReplayStatus == EMovieSceneReplayStatus::Loading)
	{
		return;
	}

	// Stop the current replay if the user just disarmed it.
	// Start the current replay if it's armed and we haven't started it yet.
	const FReplayInfo& ActiveReplayInfo = CurrentReplayInfos[0];
	if (!Manager.IsReplayArmed() && bReplayActive)
	{
		StopReplay(ActiveReplayInfo);
		return;
	}
	else if (Manager.IsReplayArmed() && !bReplayActive)
	{
		StartReplay(ActiveReplayInfo);
		return;
	}

	const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	const FSequenceInstance& Instance = InstanceRegistry->GetInstance(ActiveReplayInfo.InstanceHandle);

	TSharedRef<const FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();
	const FMovieSceneContext& Context = Instance.GetContext();

	UWorld* World = SharedPlaybackState->GetPlaybackContext()->GetWorld();
	UDemoNetDriver* DemoNetDriver = World->GetDemoNetDriver();

	if (DemoNetDriver)
	{
		AWorldSettings* WorldSettings = World->GetWorldSettings();
		FMovieSceneReplayBroker* Broker = Manager.FindBroker(World);

		const bool bIsInit = bNeedsInit;
		if (bNeedsInit)
		{
			// First evaluation since the replay started and became ready.
			// Let's notify the game-specific implementation.
			bNeedsInit = false;
			Broker->OnReplayStarted(World);
		}

		// Set time dilation and current demo time according to our current sequencer playback.
		const EMovieScenePlayerStatus::Type CurrentPlayerStatus = Context.GetStatus();

		const bool bIsPlaying = (CurrentPlayerStatus == EMovieScenePlayerStatus::Playing);
		const bool bWasPlaying = (PreviousPlayerStatus == EMovieScenePlayerStatus::Playing);

		const bool bIsPaused = (
				CurrentPlayerStatus == EMovieScenePlayerStatus::Paused ||
				CurrentPlayerStatus == EMovieScenePlayerStatus::Stopped);
		const bool bWasPaused = (
				PreviousPlayerStatus == EMovieScenePlayerStatus::Paused ||
				PreviousPlayerStatus == EMovieScenePlayerStatus::Stopped);

		if (bIsInit || (bIsPaused && !bWasPaused))
		{
			// Pause the replay.
			WorldSettings->SetTimeDilation(0.f);
			Broker->OnReplayPause(World);
		}
		else if (bIsPlaying && !bWasPlaying)
		{
			// Start playing.
			WorldSettings->SetTimeDilation(1.f);
			WorldSettings->SetPauserPlayerState(nullptr);
			Broker->OnReplayPlay(World);
		}

		PreviousPlayerStatus = CurrentPlayerStatus;

		// Update the current replay time.
		const FFrameNumber SectionStartTime = ActiveReplayInfo.Section->GetTrueRange().GetLowerBoundValue();
		const FFrameTime CurrentReplayTime = Context.GetTime() - SectionStartTime;
		const float CurrentReplayTimeInSeconds = Context.GetFrameRate().AsSeconds(CurrentReplayTime);

		const bool bIsScrubbing = Context.HasJumped() ||
			(CurrentPlayerStatus == EMovieScenePlayerStatus::Scrubbing) ||
			(CurrentPlayerStatus == EMovieScenePlayerStatus::Jumping) ||
			(CurrentPlayerStatus == EMovieScenePlayerStatus::Stepping);

		if (bIsScrubbing || bIsPaused)
		{
			// Scrub replay to the desired time.
			DemoNetDriver->GotoTimeInSeconds(CurrentReplayTimeInSeconds);
			Broker->OnGoToTime(World, CurrentReplayTimeInSeconds);
		}
		else if (bIsPlaying)
		{
			// Keep time in sync with sequencer while playing.
			DemoNetDriver->SetDemoCurrentTime(CurrentReplayTimeInSeconds);
		}

		// Set some CVars according to the playback state.
		if (ShowFlagMotionBlur)
		{
			int32 ShowMotionBlur = bIsPlaying ? 1 : 0;
			ShowFlagMotionBlur->Set(ShowMotionBlur);
		}

		// Hack some stuff for known spectator controllers.
		if (APlayerController* PlayerController = World->GetFirstPlayerController())
		{
			if (ASpectatorPawn* SpectatorPawn = PlayerController->GetSpectatorPawn())
			{
				if (USpectatorPawnMovement* SpectatorPawnMovement = Cast<USpectatorPawnMovement>(SpectatorPawn->GetMovementComponent()))
				{
					SpectatorPawnMovement->bIgnoreTimeDilation = true;
				}
			}
		}
	}
	// else: replay hasn't yet started (loading map, etc.)
}

void UMovieSceneReplaySystem::StartReplay(const FReplayInfo& ReplayInfo)
{
	using namespace UE::MovieScene;

	if (!ensure(ReplayInfo.IsValid()))
	{
		return;
	}

	const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	const FSequenceInstance& Instance = InstanceRegistry->GetInstance(ReplayInfo.InstanceHandle);
	TSharedRef<const FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();
	const FMovieSceneContext Context = Instance.GetContext();
	UWorld* World = SharedPlaybackState->GetPlaybackContext()->GetWorld();

	UDemoNetDriver* DemoNetDriver = World->GetDemoNetDriver();
	if (!ensure(DemoNetDriver == nullptr))
	{
		// Are we already inside a replay?
		return;
	}
	UGameInstance* GameInstance = World->GetGameInstance();
	if (!ensure(GameInstance != nullptr))
	{
		// We need a game instance to start the replay.
		return;
	}

	// Delay starting replay until next tick so that we don't have any problems loading a new level while we're
	// in the middle of the sequencer evaluation.
	World->GetTimerManager().SetTimerForNextTick([this, World, ReplayInfo]()
		{
			check(ReplayInfo.IsValid());
			const FString ReplayName = ReplayInfo.Section->ReplayName;
			World->GetGameInstance()->PlayReplay(ReplayName, nullptr, TArray<FString>());
		});

	// We have a few things to do just before/after the map has been loaded.
	if (!PreLoadMapHandle.IsValid())
	{
		PreLoadMapHandle = FCoreUObjectDelegates::PreLoadMap.AddStatic(UMovieSceneReplaySystem::OnPreLoadMap, SharedPlaybackState);
	}
	if (!PostLoadMapHandle.IsValid())
	{
		PostLoadMapHandle = FCoreUObjectDelegates::PostLoadMapWithWorld.AddStatic(UMovieSceneReplaySystem::OnPostLoadMap, SharedPlaybackState, Context);
	}
	if (!EndPlayMapHandle.IsValid())
	{
		EndPlayMapHandle = FGameDelegates::Get().GetEndPlayMapDelegate().AddStatic(UMovieSceneReplaySystem::OnEndPlayMap);
	}

	// Update the replay status.
	FMovieSceneReplayManager& Manager = FMovieSceneReplayManager::Get();
	Manager.ReplayStatus = EMovieSceneReplayStatus::Loading;

	bReplayActive = true;
	bNeedsInit = true;
}

void UMovieSceneReplaySystem::StopReplay(const FReplayInfo& ReplayInfo)
{
	using namespace UE::MovieScene;

	if (!ensure(ReplayInfo.IsValid() && bReplayActive))
	{
		return;
	}

	const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	const FSequenceInstance& Instance = InstanceRegistry->GetInstance(ReplayInfo.InstanceHandle);
	TSharedRef<const FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();
	UWorld* World = SharedPlaybackState->GetPlaybackContext()->GetWorld();
	UGameInstance* GameInstance = World->GetGameInstance();
	UReplaySubsystem* ReplaySubsystem = GameInstance ? GameInstance->GetSubsystem<UReplaySubsystem>() : nullptr;
	if (ReplaySubsystem != nullptr)
	{
		ReplaySubsystem->bLoadDefaultMapOnStop = false;

		World->GetTimerManager().SetTimerForNextTick([ReplaySubsystem]()
			{
				ReplaySubsystem->StopReplay();
			});
	}

	FMovieSceneReplayManager& Manager = FMovieSceneReplayManager::Get();
	Manager.ReplayStatus = EMovieSceneReplayStatus::Stopped;
	Manager.FindBroker(World)->OnReplayStopped(World);

	bReplayActive = false;
}

void UMovieSceneReplaySystem::OnPreLoadMap(const FString& MapName, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	FCoreUObjectDelegates::PreLoadMap.Remove(PreLoadMapHandle);
	PreLoadMapHandle.Reset();

	// Clear the spawn register, so that any spawnables (like cameras) can respawn immediately in the newly loaded replay map.
	// This isn't done by default because when the map gets unloaded everything gets destroyed and we don't end up being able
	// to call "Finish" on our sequence instances.
	// We could call "Finish" here but we dont' want that either because it actually re-evaluates the sequence one last time, 
	// which would re-trigger the replay and re-re-load the replay map again.
	// So we just do the minimum we need here.
	if (FMovieSceneSpawnRegister* SpawnRegister = SharedPlaybackState->FindCapability<FMovieSceneSpawnRegister>())
	{
		SpawnRegister->ForgetExternallyOwnedSpawnedObjects(SharedPlaybackState);
		SpawnRegister->CleanUp(SharedPlaybackState);
	}
}

void UMovieSceneReplaySystem::OnPostLoadMap(UWorld* World, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, FMovieSceneContext LastContext)
{
	FCoreUObjectDelegates::PostLoadMapWithWorld.Remove(PostLoadMapHandle);
	PostLoadMapHandle.Reset();

	// After the map has loaded, we wait for the game to be ready to start showing the replay. This generally includes waiting
	// for the replay pawn and player controller.
	World->GetTimerManager().SetTimer(ReEvaluateHandle, [World, SharedPlaybackState, LastContext]()
		{
			FMovieSceneReplayManager& Manager = FMovieSceneReplayManager::Get();
			FMovieSceneReplayBroker* Broker = Manager.FindBroker(World);
			if (Broker->CanStartReplay(World))
			{
				World->GetTimerManager().ClearTimer(ReEvaluateHandle);

				// Re-evaluate synchronously.
				if (TSharedPtr<FMovieSceneEntitySystemRunner> Runner = SharedPlaybackState->GetRunner())
				{
					Runner->QueueUpdate(LastContext, SharedPlaybackState->GetRootInstanceHandle());
					Runner->Flush();
				}
			}
		},
		0.1f,
		true);
}

void UMovieSceneReplaySystem::OnEndPlayMap()
{
	FGameDelegates::Get().GetEndPlayMapDelegate().Remove(EndPlayMapHandle);
	EndPlayMapHandle.Reset();

	// Disarm the replay when PIE ends since we don't want to re-start it immediately if we start PIE again.
	// It might have already been disarmed if we scrubbed past the replay section, though.
	FMovieSceneReplayManager& Manager = FMovieSceneReplayManager::Get();
	if (Manager.IsReplayArmed())
	{
		Manager.DisarmReplay();
	}
}

bool UMovieSceneReplaySystem::FReplayInfo::IsValid() const
{
	return Section != nullptr && !Section->ReplayName.IsEmpty() && InstanceHandle.IsValid();
}
