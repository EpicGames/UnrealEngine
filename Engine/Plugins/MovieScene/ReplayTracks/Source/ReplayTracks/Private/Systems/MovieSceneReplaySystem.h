// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "MovieSceneFwd.h"
#include "MovieSceneReplaySystem.generated.h"

class FDelegateHandle;
class IConsoleVariable;
struct FMovieSceneContext;
struct FTimerHandle;

class UMovieSceneReplaySection;

namespace UE
{
namespace MovieScene
{

struct FSharedPlaybackState;

struct FReplayComponentData
{
	FReplayComponentData() : Section(nullptr) {}
	FReplayComponentData(const UMovieSceneReplaySection* InSection) : Section(InSection) {}

	const UMovieSceneReplaySection* Section = nullptr;
};

struct FReplayComponentTypes
{
public:
	static FReplayComponentTypes* Get();

	TComponentTypeID<FReplayComponentData> Replay;

private:
	FReplayComponentTypes();
};

} // namespace MovieScene
} // namespace UE

UCLASS(MinimalAPI)
class UMovieSceneReplaySystem : public UMovieSceneEntitySystem
{
public:
	GENERATED_BODY()

	UMovieSceneReplaySystem(const FObjectInitializer& ObjInit);

private:
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;

	void OnRunInstantiation();
	void OnRunEvaluation();

private:

	struct FReplayInfo
	{
		const UMovieSceneReplaySection* Section = nullptr;
		UE::MovieScene::FInstanceHandle InstanceHandle;

		bool IsValid() const;

		friend bool operator==(const FReplayInfo& A, const FReplayInfo& B)
		{
			return A.Section == B.Section && A.InstanceHandle == B.InstanceHandle;
		}
		friend bool operator!=(const FReplayInfo& A, const FReplayInfo& B)
		{
			return A.Section != B.Section || A.InstanceHandle != B.InstanceHandle;
		}
	};

	void StartReplay(const FReplayInfo& ReplayInfo);
	void StopReplay(const FReplayInfo& ReplayInfo);

private:

	TArray<FReplayInfo> CurrentReplayInfos;
	bool bReplayActive = false;
	bool bNeedsInit = true;

	EMovieScenePlayerStatus::Type PreviousPlayerStatus = EMovieScenePlayerStatus::Stopped;

	IConsoleVariable* ShowFlagMotionBlur = nullptr;

private:

	// Handlers for replay events. These are all static because they happen across level reloads,
	// which mean that the current object may have been wiped out and re-created, so we can't rely
	// on any instance data.
	static void OnPreLoadMap(const FString& MapName, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState);
	static void OnPostLoadMap(UWorld* World, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, FMovieSceneContext LastContext);
	static void OnEndPlayMap();

	static FDelegateHandle PreLoadMapHandle;
	static FDelegateHandle PostLoadMapHandle;
	static FDelegateHandle EndPlayMapHandle;
	static FTimerHandle ReEvaluateHandle;
};
