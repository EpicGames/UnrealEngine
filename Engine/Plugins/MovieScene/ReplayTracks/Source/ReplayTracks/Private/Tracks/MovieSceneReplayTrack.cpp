// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneReplayTrack.h"
#include "Sections/MovieSceneReplaySection.h"
#include "MovieScene.h"

#define LOCTEXT_NAMESPACE "MovieSceneReplayTrack"

UMovieSceneReplayTrack::UMovieSceneReplayTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);
}

void UMovieSceneReplayTrack::AddSection(UMovieSceneSection& Section)
{
	if (UMovieSceneReplaySection* ReplaySection = Cast<UMovieSceneReplaySection>(&Section))
	{
		Sections.Add(ReplaySection);
	}
}

bool UMovieSceneReplayTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneReplaySection::StaticClass();
}

UMovieSceneSection* UMovieSceneReplayTrack::CreateNewSection()
{
	return NewObject<UMovieSceneReplaySection>(this, NAME_None, RF_Transactional);
}

const TArray<UMovieSceneSection*>& UMovieSceneReplayTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneReplayTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}


bool UMovieSceneReplayTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

void UMovieSceneReplayTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneReplayTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}

void UMovieSceneReplayTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}

UMovieSceneReplaySection* UMovieSceneReplayTrack::AddNewReplaySection(FFrameNumber KeyTime)
{
	UMovieSceneReplaySection* NewSection = Cast<UMovieSceneReplaySection>(CreateNewSection());

	const UMovieScene* OwnerScene = GetTypedOuter<UMovieScene>();
	const TRange<FFrameNumber> PlaybackRange = OwnerScene->GetPlaybackRange();
	NewSection->InitialPlacement(Sections, KeyTime, PlaybackRange.Size<FFrameNumber>().Value, false);

	AddSection(*NewSection);

	return NewSection;
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneReplayTrack::GetDisplayName() const
{
	return LOCTEXT("TrackName", "Replay");
}

#endif

#undef LOCTEXT_NAMESPACE
