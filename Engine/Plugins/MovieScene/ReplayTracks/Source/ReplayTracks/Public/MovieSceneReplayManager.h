// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Misc/InlineValue.h"

class UMovieSceneReplaySection;

/**
 * Base class for providing game-specific functionality required for managing replay.
 */
class REPLAYTRACKS_API FMovieSceneReplayBroker
{
public:
	virtual ~FMovieSceneReplayBroker() {}

	virtual bool SupportsWorld(const UWorld* InWorld) const { return false; }

	virtual bool CanStartReplay(const UWorld* InWorld) const { return false; }
	virtual void OnReplayStarted(UWorld* InWorld) {}

	virtual void OnGoToTime(UWorld* InWorld, float TimeInSeconds) {}
	virtual void OnReplayPlay(UWorld* InWorld) {}
	virtual void OnReplayPause(UWorld* InWorld) {}

	virtual void OnReplayStopped(UWorld* InWorld) {}
};

/**
 * Handle for a registered replay broker.
 */
struct REPLAYTRACKS_API FMovieSceneReplayBrokerHandle
{
	int32 Value = INDEX_NONE;
};

/**
 * Current status of the replay, as managed by the replay manager.
 */
enum class EMovieSceneReplayStatus
{
	Stopped,
	Loading,
	Playing
};

/**
 * Utility class for managing replay and replay brokers.
 */
class REPLAYTRACKS_API FMovieSceneReplayManager
{
public:
	// Gets the global replay manager.
	static FMovieSceneReplayManager& Get();

	// Returns whether replay is armed.
	bool IsReplayArmed() const;
	// Arms replay, i.e. indicates that it's ready to start.
	void ArmReplay();
	// Disarms replay, i.e. indicates that it's not ready anymore, or should be stopped if currently playing.
	void DisarmReplay();

	// Gets the current replay status.
	EMovieSceneReplayStatus GetReplayStatus() const { return ReplayStatus; }

	// Registers a new broker.
	template<typename BrokerClass, typename... Args>
	FMovieSceneReplayBrokerHandle RegisterBroker(Args&&... ArgList)
	{
		int32 Index = Brokers.Emplace(BrokerClass(Forward<Args>(ArgList)...));
		return FMovieSceneReplayBrokerHandle{ Index };
	}

	// Unregisters an existing broker.
	bool UnregisterBroker(FMovieSceneReplayBrokerHandle InHandle);

	// Finds the broker appropriate for a given world.
	FMovieSceneReplayBroker* FindBroker(const UWorld* InWorld) const;

private:
	FMovieSceneReplayManager();
	FMovieSceneReplayManager(const FMovieSceneReplayManager&) = delete;
	FMovieSceneReplayManager& operator=(const FMovieSceneReplayManager& In) = delete;

	bool bIsReplayArmed = false;
	EMovieSceneReplayStatus ReplayStatus = EMovieSceneReplayStatus::Stopped;

	TArray<TInlineValue<FMovieSceneReplayBroker>, TInlineAllocator<4>> Brokers;

	friend class UMovieSceneReplaySystem;
};

