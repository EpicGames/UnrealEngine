// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "MovieSceneSection.h"
#include "MovieSceneReplaySection.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneReplaySection 
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
public:

	GENERATED_BODY()

	UMovieSceneReplaySection(const FObjectInitializer& ObjInitializer);

private:

	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

public:

	/** The name of the replay to run */
	UPROPERTY(EditAnywhere, Category="Replay")
	FString ReplayName;
};
