// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/InlineValue.h"
#include "MovieSceneTrack.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneReplayTrack.generated.h"

class UMovieSceneReplaySection;

UCLASS(MinimalAPI)
class UMovieSceneReplayTrack
	: public UMovieSceneTrack
{
public:
	GENERATED_BODY()

	UMovieSceneReplayTrack(const FObjectInitializer& ObjectInitializer);

	REPLAYTRACKS_API UMovieSceneReplaySection* AddNewReplaySection(FFrameNumber KeyTime);

	// UMovieSceneTrack interface
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual bool IsEmpty() const override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual void RemoveAllAnimationData() override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const override;
#endif
private:

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};

