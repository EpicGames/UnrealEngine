// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplayTracksEditorModule.h"
#include "Camera/CameraComponent.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneCommonHelpers.h"
#include "TrackEditors/ReplayTrackEditor.h"

#define LOCTEXT_NAMESPACE "ReplayTracksEditorModule"

void UReplayTracksCameraModifier::SetLockedActor(AActor* InLockedActor)
{
	LockedActor = InLockedActor;
}

void UReplayTracksCameraModifier::ModifyCamera(float DeltaTime, FVector ViewLocation, FRotator ViewRotation, float FOV, FVector& NewViewLocation, FRotator& NewViewRotation, float& NewFOV)
{
	// TODO: the aspect ratio constraint will be wrong! Because camera modifiers can't affect it!
	if (AActor* Actor = LockedActor.Get())
	{
		FMinimalViewInfo ViewInfo;
		ViewInfo.Location = ViewLocation;
		ViewInfo.Rotation = ViewRotation;
		ViewInfo.FOV = FOV;
		
		Actor->CalcCamera(DeltaTime, ViewInfo);

		// We only use this camera modifier to get the locked actor's FOV and post-process settings.
		// The location and rotation are left alone so that the user can fly away at any time.
		NewFOV = ViewInfo.FOV;
	}
}

void UReplayTracksCameraModifier::ModifyPostProcess(float DeltaTime, float& PostProcessBlendWeight, FPostProcessSettings& PostProcessSettings)
{
	if (AActor* Actor = LockedActor.Get())
	{
		if (UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromActor(Actor))
		{
			PostProcessSettings = CameraComponent->PostProcessSettings;
			PostProcessBlendWeight = CameraComponent->PostProcessBlendWeight;
		}
	}
}

/**
 * Implements the FReplayTracksEditorModule module.
 */
class FReplayTracksEditorModule : public IReplayTracksEditorModule
{
public:
	FReplayTracksEditorModule()
	{
	}

	virtual void StartupModule() override
	{
		RegisterTrackEditors();
	}

	virtual void ShutdownModule() override
	{
		UnregisterTrackEditors();
	}

	virtual void SetLockedActor(UWorld* World, AActor* LockedActor) override
	{
		APlayerController* PlayerController = World->GetFirstPlayerController();

		if (LockedActor)
		{
			if (!LockedActorCameraModifier.IsValid())
			{
				LockedActorCameraModifier = CastChecked<UReplayTracksCameraModifier>(PlayerController->PlayerCameraManager->AddNewCameraModifier(UReplayTracksCameraModifier::StaticClass()));
			}
			LockedActorCameraModifier->SetLockedActor(LockedActor);
		}
		else
		{
			if (LockedActorCameraModifier.IsValid())
			{
				LockedActorCameraModifier->SetLockedActor(nullptr);
			}
		}
	}

private:
	void RegisterTrackEditors()
	{
		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		ReplayTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FReplayTrackEditor::CreateTrackEditor));
	}

	void UnregisterTrackEditors()
	{
		ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModule)
		{
			SequencerModule->UnRegisterTrackEditor(ReplayTrackCreateEditorHandle);
		}
	}

private:
	FDelegateHandle ReplayTrackCreateEditorHandle;
	TWeakObjectPtr<UReplayTracksCameraModifier> LockedActorCameraModifier;
};

IMPLEMENT_MODULE(FReplayTracksEditorModule, ReplayTracksEditor);

#undef LOCTEXT_NAMESPACE
