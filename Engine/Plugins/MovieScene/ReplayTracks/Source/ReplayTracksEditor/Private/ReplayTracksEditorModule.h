// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraModifier.h"
#include "Modules/ModuleInterface.h"
#include "ReplayTracksEditorModule.generated.h"

/**
 * Camera modifier that lets us set the correct FOV and post-process settings on the spectator pawn
 * when we want to lock into a camera.
 */
UCLASS()
class UReplayTracksCameraModifier : public UCameraModifier
{
	GENERATED_BODY()

public:
	void SetLockedActor(AActor* InLockedActor);

protected:
	virtual void ModifyCamera(float DeltaTime, FVector ViewLocation, FRotator ViewRotation, float FOV, FVector& NewViewLocation, FRotator& NewViewRotation, float& NewFOV) override;
	virtual void ModifyPostProcess(float DeltaTime, float& PostProcessBlendWeight, FPostProcessSettings& PostProcessSettings) override;

private:
	TWeakObjectPtr<AActor> LockedActor;
};

/**
 * The interface for the replay tracks module.
 */
class IReplayTracksEditorModule : public IModuleInterface
{
public:
	virtual void SetLockedActor(UWorld* World, AActor* LockedActor) = 0;
};
