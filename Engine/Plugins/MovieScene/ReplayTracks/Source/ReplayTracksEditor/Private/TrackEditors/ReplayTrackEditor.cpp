// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/ReplayTrackEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/SpectatorPawn.h"
#include "LevelEditorViewport.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneReplayManager.h"
#include "ReplayTracksEditorModule.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneReplaySection.h"
#include "SequencerUtilities.h"
#include "Tracks/MovieSceneReplayTrack.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ReplayTrackEditor"

FReplayTrackEditor::FReplayTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{
}

TSharedRef<ISequencerTrackEditor> FReplayTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FReplayTrackEditor(InSequencer));
}

TSharedPtr<SWidget> FReplayTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			FSequencerUtilities::MakeAddButton(LOCTEXT("AddReplayButton", "Replay"), FOnGetContent::CreateSP(this, &FReplayTrackEditor::HandleAddReplayTrackComboButtonGetMenuContent), Params.NodeIsHovered, GetSequencer())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ContentPadding(FMargin(5, 2))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ForegroundColor(FSlateColor::UseForeground())
			.IsEnabled(this, &FReplayTrackEditor::HandleToggleReplayButtonIsEnabled)
			.ToolTipText(this, &FReplayTrackEditor::HandleToggleReplayButtonToolTipText)
			.OnClicked(this, &FReplayTrackEditor::HandleToggleReplayButtonClicked)
			[
				SNew(STextBlock)
				.Text(this, &FReplayTrackEditor::HandleGetToggleReplayButtonContent)
			]
		];
}

TSharedRef<ISequencerSection> FReplayTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check( SupportsType( SectionObject.GetOuter()->GetClass() ) );
	
	return MakeShareable(new FReplaySection(GetSequencer(), *CastChecked<UMovieSceneReplaySection>(&SectionObject)));
}

bool FReplayTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneReplayTrack::StaticClass();
}

void FReplayTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddReplayTrack", "Replay Track"),
		LOCTEXT("AddReplayTrackTooltip", "Adds a replay track."),
		FSlateIcon(), //FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.CameraCut"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FReplayTrackEditor::HandleAddReplayTrackMenuEntryExecute),
			FCanExecuteAction::CreateRaw(this, &FReplayTrackEditor::HandleAddReplayTrackMenuEntryCanExecute)
		)
	);
}

bool FReplayTrackEditor::HandleAddReplayTrackMenuEntryCanExecute() const
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	return ((FocusedMovieScene != nullptr) && (FocusedMovieScene->FindTrack<UMovieSceneReplayTrack>() == nullptr));
}

void FReplayTrackEditor::HandleAddReplayTrackMenuEntryExecute()
{
	UMovieSceneReplayTrack* ReplayTrack = FindOrCreateReplayTrack();

	if (ReplayTrack)
	{
		if (GetSequencer().IsValid())
		{
			GetSequencer()->OnAddTrack(ReplayTrack, FGuid());
		}
	}
}

FText FReplayTrackEditor::HandleGetToggleReplayButtonContent() const
{
	FMovieSceneReplayManager& Manager = FMovieSceneReplayManager::Get();
	const bool bIsArmed = Manager.IsReplayArmed();
	switch (Manager.GetReplayStatus())
	{
		case EMovieSceneReplayStatus::Stopped:
		default:
			return bIsArmed ? 
					LOCTEXT("ReplayButtonArmed", "Ready for Replay") :
					LOCTEXT("ReplayButtonDisarmed", "Start Replay");
		case EMovieSceneReplayStatus::Loading:
			return LOCTEXT("ReplayButtonLoading", "Loading Replay");
		case EMovieSceneReplayStatus::Playing:
			return LOCTEXT("ReplayButtonPlaying", "Stop Replay");
	}
}

FSlateColor FReplayTrackEditor::HandleGetToggleReplayButtonColor() const
{
	FMovieSceneReplayManager& Manager = FMovieSceneReplayManager::Get();
	if (Manager.IsReplayArmed())
	{
		return FSlateColor(FLinearColor(1.f, 0.1f, 0.f, 0.6f));
	}
	return FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.4f));
}

bool FReplayTrackEditor::HandleToggleReplayButtonIsEnabled() const
{
	UWorld* PlaybackWorld = GetPIEPlaybackWorld();
	return PlaybackWorld != nullptr;
}

FText FReplayTrackEditor::HandleToggleReplayButtonToolTipText() const
{
	UWorld* PlaybackWorld = GetPIEPlaybackWorld();
	if (!PlaybackWorld)
	{
		return LOCTEXT("ReplayButtonDisabledToolTip", "Please start PIE before starting the replay");
	}
	
	FMovieSceneReplayManager& Manager = FMovieSceneReplayManager::Get();
	const bool bIsArmed = Manager.IsReplayArmed();
	switch (Manager.GetReplayStatus())
	{
		case EMovieSceneReplayStatus::Stopped:
		default:
			return LOCTEXT("ReplayButtonArmedToolTip", "Start the replay in PIE");
		case EMovieSceneReplayStatus::Loading:
			return LOCTEXT("ReplayButtonLoadingToolTip", "The replay is currently loading, please wait");
		case EMovieSceneReplayStatus::Playing:
			return LOCTEXT("ReplayButtonPlayingToolTip", "Replay is active, press the button to stop it");
	}
}

FReply FReplayTrackEditor::HandleToggleReplayButtonClicked()
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid())
	{
		FMovieSceneReplayManager& Manager = FMovieSceneReplayManager::Get();
		if (!Manager.IsReplayArmed())
		{
			Manager.ArmReplay();
		}
		else
		{
			Manager.DisarmReplay();
		}
		SequencerPtr->ForceEvaluate();
	}
	return FReply::Handled();
}

UMovieSceneReplayTrack* FReplayTrackEditor::FindOrCreateReplayTrack(bool* bWasCreated)
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (FocusedMovieScene->IsReadOnly())
	{
		return nullptr;
	}

	UMovieSceneReplayTrack* ReplayTrack = FocusedMovieScene->FindTrack<UMovieSceneReplayTrack>();

	if (ReplayTrack == nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddReplayTrack_Transaction", "Add Replay Track"));
		FocusedMovieScene->Modify();
		
		ReplayTrack = FocusedMovieScene->AddTrack<UMovieSceneReplayTrack>();

		if (bWasCreated)
		{
			*bWasCreated = true;
		}
	}

	return ReplayTrack;
}

TSharedRef<SWidget> FReplayTrackEditor::HandleAddReplayTrackComboButtonGetMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddReplaySection", "Add Replay"),
		LOCTEXT("AddReplaySectionTooltip", "Adds a replay section"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() { 
				AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FReplayTrackEditor::AddKeyInternal));
			}))
		);

	return MenuBuilder.MakeWidget();
}

FKeyPropertyResult FReplayTrackEditor::AddKeyInternal(FFrameNumber KeyTime)
{
	FKeyPropertyResult KeyPropertyResult;

	UMovieSceneReplayTrack* ReplayTrack = FindOrCreateReplayTrack(&KeyPropertyResult.bTrackCreated);
	const TArray<UMovieSceneSection*>& AllSections = ReplayTrack->GetAllSections();

	UMovieSceneReplaySection* NewSection = ReplayTrack->AddNewReplaySection(KeyTime);
	KeyPropertyResult.bTrackModified = true;

	GetSequencer()->EmptySelection();
	GetSequencer()->SelectSection(NewSection);
	GetSequencer()->ThrobSectionSelection();

	return KeyPropertyResult;
}

void FReplayTrackEditor::OnInitialize()
{
	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	GlobalTimeChangedHandle = SequencerPtr->OnGlobalTimeChanged().Add(FSimpleDelegate::CreateSP(this, &FReplayTrackEditor::OnGlobalTimeChanged));
}

void FReplayTrackEditor::OnRelease()
{
	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid())
	{
		SequencerPtr->OnGlobalTimeChanged().Remove(GlobalTimeChangedHandle);
	}
}

void FReplayTrackEditor::Tick(float DeltaTime)
{
	if (UWorld* World = GetPIEPlaybackWorld())
	{
		MoveLockedActorsToPIEViewTarget(World);
	}
}

void FReplayTrackEditor::OnGlobalTimeChanged()
{
	// When the sequencer has updated, move the view target to where the locked actor has been animated.
	if (UWorld* World = GetPIEPlaybackWorld())
	{
		MovePIEViewTargetToLockedActor(World);
	}
}

UWorld* FReplayTrackEditor::GetPIEPlaybackWorld() const
{
	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	if (!ensure(SequencerPtr.IsValid()))
	{
		return nullptr;
	}

	const UObject* PlaybackContext = SequencerPtr->GetPlaybackContext();
	UWorld* PlaybackWorld = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
	if (PlaybackWorld && PlaybackWorld->WorldType != EWorldType::PIE && PlaybackWorld->WorldType != EWorldType::Game)
	{
		return nullptr;
	}

	return PlaybackWorld;
}

AActor* FReplayTrackEditor::GetPIEViewportLockedActor()
{
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		// TOOD: This is probably the wrong way to do this, but it will be easier when the replay doesn't change the map from under the viewport.
		if (LevelVC != nullptr && LevelVC->AllowsCinematicControl())
		{
			const FLevelViewportActorLock& ActorLock = LevelVC->GetActorLock();
			const bool bIsLocked = LevelVC->bLockedCameraView;
			AActor* LockedActor = ActorLock.GetLockedActor();
			if (LockedActor && bIsLocked)
			{
				return LockedActor;
			}
		}
	}
	return nullptr;
}

void FReplayTrackEditor::MoveLockedActorsToPIEViewTarget(UWorld* PlaybackWorld)
{
	if (!PlaybackWorld)
	{
		return;
	}

	bool bMoveLockedActor = true;
	AActor* LockedActor = GetPIEViewportLockedActor();

	if (LockedActor != LastLockedActor.Get())
	{
		// We have changed locked actors! Move the view target to the locked actor instead of the other way around,
		// but just for this frame. We do this by re-evaluating the sequence.
		GetSequencer()->RequestEvaluate();
		bMoveLockedActor = false;
	}

	// Update our last known locked actor.
	LastLockedActor = LockedActor;

	// Update the locked actor with our camera modifier.
	IReplayTracksEditorModule& ReplayTracksEditorModule = FModuleManager::LoadModuleChecked<IReplayTracksEditorModule>("ReplayTracksEditor");
	ReplayTracksEditorModule.SetLockedActor(PlaybackWorld, LastLockedActor.Get());

	// Move any locked actor(s) to 
	const APlayerController* PlayerController = PlaybackWorld->GetFirstPlayerController();
	if (bMoveLockedActor && LockedActor != nullptr && PlayerController != nullptr && PlayerController->PlayerCameraManager != nullptr)
	{
		if (USceneComponent* LockedActorSceneComponent = LockedActor->GetRootComponent())
		{
			FVector Translation;
			FRotator Rotation;
			// TODO: should call PlayerCameraManager::GetCachedCameraPOV ?
			PlayerController->GetPlayerViewPoint(Translation, Rotation);

			// We need to normalize this otherwise we get angles in the wrong space, which adds 360 flips when keying.
			Rotation.Normalize();

			// Move the locked actor.
			const bool bIsSimulatingPhysics = LockedActorSceneComponent ? LockedActorSceneComponent->IsSimulatingPhysics() : false;
			LockedActorSceneComponent->SetRelativeLocationAndRotation(Translation, Rotation, false, nullptr, bIsSimulatingPhysics ? ETeleportType::ResetPhysics : ETeleportType::None);
			// Force the location and rotation values to avoid Rot->Quat->Rot conversions
			LockedActorSceneComponent->SetRelativeLocation_Direct(Translation);
			LockedActorSceneComponent->SetRelativeRotation_Direct(Rotation);
		}
	}
}

void FReplayTrackEditor::MovePIEViewTargetToLockedActor(UWorld* PlaybackWorld)
{
	if (!PlaybackWorld)
	{
		return;
	}

	AActor* LockedActor = GetPIEViewportLockedActor();

	APlayerController* PlayerController = PlaybackWorld->GetFirstPlayerController();
	if (LockedActor != nullptr && PlayerController != nullptr && PlayerController->PlayerCameraManager != nullptr)
	{
		AActor* ViewTarget = PlayerController->PlayerCameraManager->GetViewTarget();
		USceneComponent* ViewTargetSceneComponent = ViewTarget ? ViewTarget->GetRootComponent() : nullptr;

		if (ViewTargetSceneComponent)
		{
			const USceneComponent* LockedActorSceneComponent = LockedActor->GetRootComponent();
			const FTransform LockedTargetTransform = LockedActorSceneComponent->GetComponentTransform();

			const FVector Translation = LockedTargetTransform.GetTranslation();
			const FRotator Rotation = LockedTargetTransform.GetRotation().Rotator();

			// Move the root component of the view target.
			const bool bIsSimulatingPhysics = ViewTargetSceneComponent ? ViewTargetSceneComponent->IsSimulatingPhysics() : false;
			ViewTargetSceneComponent->SetRelativeLocationAndRotation(Translation, Rotation, false, nullptr, bIsSimulatingPhysics ? ETeleportType::ResetPhysics : ETeleportType::None);
			// Force the location and rotation values to avoid Rot->Quat->Rot conversions
			ViewTargetSceneComponent->SetRelativeLocation_Direct(Translation);
			ViewTargetSceneComponent->SetRelativeRotation_Direct(Rotation);

			// Rotate the pawn... most of the time that's going to rotate the same thing we just rotated above (ViewTargetSceneComponent)
			// but it's not necessarily the case.
			if (ASpectatorPawn* SpectatorPawn = PlayerController->GetSpectatorPawn())
			{
				SpectatorPawn->SetActorRotation(Rotation);
			}

			// Rotate the player controller, because the view target doesn't always solely define the final camera view. Most of the time,
			// the player controller's own rotation is inserted into the mix.
			PlayerController->SetInitialLocationAndRotation(Translation, Rotation);
		}
	}
}

FReplaySection::FReplaySection(TSharedPtr<ISequencer> InSequencer, UMovieSceneReplaySection& InSection)
	: FSequencerSection(InSection)
{
}

#undef LOCTEXT_NAMESPACE
