// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"
#include "MovieSceneTrackEditor.h"

struct FBuildEditWidgetParams;

class UMovieSceneReplaySection;
class UMovieSceneReplayTrack;

class FReplayTrackEditor : public FMovieSceneTrackEditor
{
public:
	FReplayTrackEditor(TSharedRef<ISequencer> InSequencer);

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:
	// ISequencerTrackEditor interface
	TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool SupportsType(TSubclassOf<class UMovieSceneTrack> TrackClass) const override;
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	
	virtual void OnInitialize() override;
	virtual void OnRelease() override;
	
	virtual void Tick(float DeltaTime) override;

private:
	bool HandleAddReplayTrackMenuEntryCanExecute() const;
	void HandleAddReplayTrackMenuEntryExecute();

	UMovieSceneReplayTrack* FindOrCreateReplayTrack(bool* bWasCreated = nullptr);

	TSharedRef<SWidget> HandleAddReplayTrackComboButtonGetMenuContent();
	FText HandleGetToggleReplayButtonContent() const;
	FSlateColor HandleGetToggleReplayButtonColor() const;
	bool HandleToggleReplayButtonIsEnabled() const;
	FText HandleToggleReplayButtonToolTipText() const;
	FReply HandleToggleReplayButtonClicked();

	FKeyPropertyResult AddKeyInternal(FFrameNumber AutoKeyTime);

	void OnGlobalTimeChanged();

	UWorld* GetPIEPlaybackWorld() const;
	AActor* GetPIEViewportLockedActor();
	void MoveLockedActorsToPIEViewTarget(UWorld* PlaybackWorld);
	void MovePIEViewTargetToLockedActor(UWorld* PlaybackWorld);

private:
	TWeakObjectPtr<AActor> LastLockedActor;

	FDelegateHandle GlobalTimeChangedHandle;
};

class FReplaySection
	: public FSequencerSection
	, public TSharedFromThis<FReplaySection>
{
public:
	FReplaySection(TSharedPtr<ISequencer> InSequencer, UMovieSceneReplaySection& InSection);
	virtual ~FReplaySection() {}
};
