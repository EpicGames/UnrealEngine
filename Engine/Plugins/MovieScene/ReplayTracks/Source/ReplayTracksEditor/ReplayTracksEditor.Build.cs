// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ReplayTracksEditor : ModuleRules
{
	public ReplayTracksEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ReplayTracks",
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"MovieScene",
				"MovieSceneTools",
				"MovieSceneTracks",
				"Sequencer",				
				"Slate",
				"SlateCore",
				"UnrealEd",
				"TimeManagement",
			}
		);
	}
}
